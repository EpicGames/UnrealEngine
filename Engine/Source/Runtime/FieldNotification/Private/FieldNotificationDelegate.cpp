// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationDelegate.h"

#include "UObject/Object.h"


namespace UE::FieldNotification
{

FDelegateHandle FFieldMulticastDelegate::Add(const UObject* InObject, FFieldId InFieldId, FDelegate InNewDelegate)
{
	if (!InNewDelegate.IsBound())
	{
		return FDelegateHandle();
	}

	FDelegateHandle Result = InNewDelegate.GetHandle();
	FInvocationKey InvocationKey{ InObject, InFieldId, NAME_None };
	if (DelegateLockCount > 0)
	{
		const int32 Index = Delegates.Emplace(FInvocationElement{ InvocationKey, MoveTemp(InNewDelegate) });
		AddedEmplaceAt = FMath::Min(AddedEmplaceAt, (uint16)Index);
	}
	else
	{
		const int32 FoundIndex = UpperBound(InvocationKey);
		Delegates.EmplaceAt(FoundIndex, FInvocationElement{ InvocationKey, MoveTemp(InNewDelegate) });
	}
	return Result;
}


FDelegateHandle FFieldMulticastDelegate::Add(const UObject* InObject, FFieldId InFieldId, const FDynamicDelegate& InDynamicDelegate)
{
	if (!InDynamicDelegate.IsBound())
	{
		return FDelegateHandle();
	}

	FDelegate NewDelegate = FDelegate::CreateUFunction(const_cast<UObject*>(InDynamicDelegate.GetUObject()), InDynamicDelegate.GetFunctionName());
	FDelegateHandle Result = NewDelegate.GetHandle();
	FInvocationKey InvocationKey{ InObject, InFieldId, InDynamicDelegate.GetFunctionName()};
	if (DelegateLockCount > 0)
	{
		const int32 Index = Delegates.Emplace(FInvocationElement{ InvocationKey, MoveTemp(NewDelegate) });
		AddedEmplaceAt = FMath::Min(AddedEmplaceAt, (uint16)Index);
	}
	else
	{
		const int32 FoundIndex = UpperBound(InvocationKey);
		Delegates.EmplaceAt(FoundIndex, FInvocationElement{ InvocationKey, MoveTemp(NewDelegate) });
	}
	return Result;
}


void FFieldMulticastDelegate::RemoveElement(FInvocationElement& Element, int32 Index, FRemoveResult& Result)
{
	Result.Object = Element.Key.Object.Get();
	Result.FieldId = Element.Key.Id;
	Result.bRemoved = Element.Delegate.IsBound();
	if (DelegateLockCount > 0)
	{
		Element.Delegate.Unbind();
		++CompactionCount;
	}
	else
	{
		Delegates.RemoveAt(Index);
	}
}


void FFieldMulticastDelegate::CompleteRemove(FRemoveResult& Result) const
{
	if (Result.FieldId.IsValid())
	{
		// Search in the normal sorted list
		const int32 FoundIndex = UpperBound(Result.FieldId) - 1;
		if (Delegates.IsValidIndex(FoundIndex))
		{
			for (int32 Index = FoundIndex; Index >= 0; --Index)
			{
				const FInvocationElement& Element = Delegates[Index];
				if (Element.Key.Id != Result.FieldId)
				{
					break;
				}
				else if (Element.Key.Object == Result.Object && Element.Delegate.IsBound())
				{
					Result.bHasOtherBoundDelegates = true;
					break;
				}
			}
		}

		if (!Result.bHasOtherBoundDelegates)
		{
			for (int32 Index = AddedEmplaceAt; Index < Delegates.Num(); ++Index)
			{
				const FInvocationElement& Element = Delegates[Index];
				if (Element.Key.Id == Result.FieldId && Element.Key.Object == Result.Object && Element.Delegate.IsBound())
				{
					Result.bHasOtherBoundDelegates = true;
					break;
				}
			}
		}
	}
}


FFieldMulticastDelegate::FRemoveResult FFieldMulticastDelegate::Remove(FDelegateHandle InDelegate)
{
	FRemoveResult Result;

	if (!InDelegate.IsValid())
	{
		return Result;
	}

	for (int32 Index = Delegates.Num() - 1; Index >= 0; --Index)
	{
		FInvocationElement& Element = Delegates[Index];
		if (Element.Delegate.GetHandle() == InDelegate)
		{
			RemoveElement(Element, Index, Result);
			break;
		}
	}

	CompleteRemove(Result);

	return Result;
}


FFieldMulticastDelegate::FRemoveResult FFieldMulticastDelegate::Remove(const FDynamicDelegate& InDynamicDelegate)
{
	FRemoveResult Result;

	if (!InDynamicDelegate.IsBound())
	{
		// The UObject may be GCed. We would like to compact the list but there is not way to know which one (since HasSameObject will return false).
		//Result may indicate that it was not remove even if it will be removed later when we finally compact.
		return Result;
	}

	for (int32 Index = Delegates.Num() - 1; Index >= 0; --Index)
	{
		FInvocationElement& Element = Delegates[Index];
		if (Element.Key.DynamicName == InDynamicDelegate.GetFunctionName())
		{
			if (Element.Delegate.IsBoundToObject(InDynamicDelegate.GetUObject()))
			{
				RemoveElement(Element, Index, Result);
				break;
			}
		}
	}

	CompleteRemove(Result);

	return Result;
}


template<typename TRemoveOrUnbind>
void FFieldMulticastDelegate::RemoveFrom(TRemoveOrUnbind& RemoveOrUnbind, FFieldId InFieldId, bool& bFieldPresent)
{
	// Search in the normal sorted list
	const int32 FoundIndex = UpperBound(InFieldId) - 1;
	if (Delegates.IsValidIndex(FoundIndex))
	{
		for (int32 Index = FoundIndex; Index >= 0; --Index)
		{
			FInvocationElement& Element = Delegates[Index];
			if (Element.Key.Id != InFieldId)
			{
				break;
			}
			if (!RemoveOrUnbind(Element, Index))
			{
				break;
			}
		}
	}

	// The item may be at the end of the array, if added while broadcasting
	if (!bFieldPresent)
	{
		for (int32 Index = Delegates.Num() - 1; Index >= AddedEmplaceAt; --Index)
		{
			FInvocationElement& Element = Delegates[Index];
			if (Element.Key.Id == InFieldId)
			{
				if (!RemoveOrUnbind(Element, Index))
				{
					break;
				}
			}
		}
	}
}


FFieldMulticastDelegate::FRemoveFromResult FFieldMulticastDelegate::RemoveFrom(const UObject* InObject, FFieldId InFieldId, FDelegateHandle InDelegate)
{
	bool bRemoved = false;
	bool bFieldPresent = false;

	if (!InDelegate.IsValid())
	{
		return FRemoveFromResult{ bRemoved, bFieldPresent };
	}

	FFieldMulticastDelegate* Self = this;
	auto RemoveOrUnbind = [Self, &InDelegate, &bRemoved, &bFieldPresent, InObject](FInvocationElement& Element, int32 Index)
	{
		if (Element.Delegate.GetHandle() == InDelegate)
		{
			if (Self->DelegateLockCount > 0)
			{
				Element.Delegate.Unbind();
				++Self->CompactionCount;
			}
			else
			{
				Self->Delegates.RemoveAt(Index);
			}
			bRemoved = true;
			return !bFieldPresent;
		}
		if (Element.Key.Object == InObject && Element.Delegate.IsBound())
		{
			bFieldPresent = true;
			return !bRemoved;
		}
		return true;
	};

	RemoveFrom(RemoveOrUnbind, InFieldId, bFieldPresent);

	return FRemoveFromResult{ bRemoved, bFieldPresent };
}


FFieldMulticastDelegate::FRemoveFromResult FFieldMulticastDelegate::RemoveFrom(const UObject* InObject, FFieldId InFieldId, const FDynamicDelegate& InDynamicDelegate)
{
	bool bRemoved = false;
	bool bFieldPresent = false;

	if (!InDynamicDelegate.IsBound())
	{
		return FRemoveFromResult{ bRemoved, bFieldPresent };
	}

	FFieldMulticastDelegate* Self = this;
	auto RemoveOrUnbind = [Self, &InDynamicDelegate, &bRemoved, &bFieldPresent, InObject](FInvocationElement& Element, int32 Index)
	{
		if (Element.Key.DynamicName == InDynamicDelegate.GetFunctionName())
		{
			if (Element.Delegate.IsBoundToObject(InDynamicDelegate.GetUObject()))
			{
				if (Self->DelegateLockCount > 0)
				{
					Element.Delegate.Unbind();
					++Self->CompactionCount;
				}
				else
				{
					Self->Delegates.RemoveAt(Index);
				}
				bRemoved = true;
				return !bFieldPresent;
			}
		}
		if (Element.Key.Object == InObject && Element.Delegate.IsBound())
		{
			bFieldPresent = true;
			return !bRemoved;
		}
		return true;
	};

	RemoveFrom(RemoveOrUnbind, InFieldId, bFieldPresent);

	return FRemoveFromResult{ bRemoved, bFieldPresent };
}


FFieldMulticastDelegate::FRemoveAllResult FFieldMulticastDelegate::RemoveAll(const UObject* InObject, FDelegateUserObjectConst InUserObject)
{
	FRemoveAllResult Result;

	auto SetHasFields = [&Result, InObject](FInvocationElement& Element)
	{
		Result.HasFields.PadToNum(Element.Key.Id.GetIndex() + 1, false);
		Result.HasFields[Element.Key.Id.GetIndex()] = true;
	};

	if (DelegateLockCount > 0)
	{
		for (FInvocationElement& Element : Delegates)
		{
			if (Element.Key.Object == InObject)
			{
				if (Element.Delegate.IsBoundToObject(InUserObject))
				{
					Element.Delegate.Unbind();
					++CompactionCount;
					++Result.RemoveCount;
				}
				else if (!Element.Delegate.IsCompactable())
				{
					SetHasFields(Element);
				}
			}
		}
	}
	else
	{
		for (int32 Index = Delegates.Num() - 1; Index >= 0; --Index)
		{
			FInvocationElement& Element = Delegates[Index];
			if (Element.Delegate.IsCompactable())
			{
				Delegates.RemoveAt(Index);
			}
			else if (Element.Key.Object == InObject)
			{
				if (Element.Delegate.IsBoundToObject(InUserObject))
				{
					Delegates.RemoveAt(Index);
					++Result.RemoveCount;
				}
				else
				{
					SetHasFields(Element);
				}
			}
		}
		CompactionCount = 0;
	}

	return Result;
}


FFieldMulticastDelegate::FRemoveAllResult FFieldMulticastDelegate::RemoveAll(const UObject* InObject, FFieldId InFieldId, FDelegateUserObjectConst InUserObject)
{
	FRemoveAllResult Result;

	auto SetHasFields = [&Result, InObject](FInvocationElement& Element)
	{
		Result.HasFields.PadToNum(Element.Key.Id.GetIndex() + 1, false);
		Result.HasFields[Element.Key.Id.GetIndex()] = true;
	};

	if (DelegateLockCount > 0)
	{
		for (FInvocationElement& Element : Delegates)
		{
			if (Element.Key.Object == InObject)
			{
				if (Element.Key.Id == InFieldId && Element.Delegate.IsBoundToObject(InUserObject))
				{
					Element.Delegate.Unbind();
					++CompactionCount;
					++Result.RemoveCount;
				}
				else if (!Element.Delegate.IsCompactable())
				{
					SetHasFields(Element);
				}
			}
		}
	}
	else
	{
		for (int32 Index = Delegates.Num() - 1; Index >= 0; --Index)
		{
			FInvocationElement& Element = Delegates[Index];
			if (Element.Delegate.IsCompactable())
			{
				Delegates.RemoveAt(Index);
			}
			else if (Element.Key.Object == InObject)
			{
				if (Element.Key.Id == InFieldId && Element.Delegate.IsBoundToObject(InUserObject))
				{
					Delegates.RemoveAt(Index);
					++Result.RemoveCount;
				}
				else
				{
					SetHasFields(Element);
				}
			}
		}
		CompactionCount = 0;
	}

	return Result;
}


void FFieldMulticastDelegate::Broadcast(UObject* InObject, UE::FieldNotification::FFieldId InFieldId)
{
	++DelegateLockCount;

	// Search in the normal sorted list
	const int32 FoundIndex = UpperBound(InFieldId) - 1;
	if (Delegates.IsValidIndex(FoundIndex))
	{
		for (int32 Index = FoundIndex; Index >= 0; --Index)
		{
			FInvocationElement& Element = Delegates[Index];
			if (Element.Key.Id != InFieldId)
			{
				break;
			}
			if (Element.Key.Object == InObject)
			{
				Element.Delegate.ExecuteIfBound(InObject, InFieldId);
			}
		}
	}

	// The item may be at the end of the array, if added while broadcasting
	for (int32 Index = Delegates.Num() - 1; Index >= AddedEmplaceAt; --Index)
	{
		FInvocationElement& Element = Delegates[Index];
		if (Element.Key.Id == InFieldId && Element.Key.Object == InObject)
		{
			Element.Delegate.ExecuteIfBound(InObject, InFieldId);
		}
	}

	--DelegateLockCount;
	ExecuteLockOperations();
}


void FFieldMulticastDelegate::Reset()
{
	if (DelegateLockCount > 0)
	{
		for (FInvocationElement& Element : Delegates)
		{
			CompactionCount += Element.Delegate.IsBound() ? 1 : 0;
			Element.Delegate.Unbind();
		}
	}
	else
	{
		Delegates.Reset();
		DelegateLockCount = 0;
		CompactionCount = 0;
		AddedEmplaceAt = std::numeric_limits<uint16>::max();
	}
}


void FFieldMulticastDelegate::ExecuteLockOperations()
{
	if (DelegateLockCount <= 0)
	{
		// Remove items that got removed while broadcasting
		if (CompactionCount > 2)
		{
			for (int32 Index = Delegates.Num() - 1; Index >= 0; --Index)
			{
				FInvocationElement& Element = Delegates[Index];
				if (Element.Delegate.IsCompactable())
				{
					Delegates.RemoveAt(Index);
					if (AddedEmplaceAt != std::numeric_limits<uint16>::max() && Index < AddedEmplaceAt)
					{
						check(AddedEmplaceAt > 0);
						--AddedEmplaceAt;
					}
				}
			}
			CompactionCount = 0;
		}

		// Sort item that were added while broadcasting
		for (; AddedEmplaceAt < Delegates.Num();)
		{
			if (Delegates[AddedEmplaceAt].Delegate.IsBound())
			{
				const int32 FoundIndex = UpperBound(Delegates[AddedEmplaceAt].Key);
				if (FoundIndex != AddedEmplaceAt)
				{
					FInvocationElement Element = MoveTemp(Delegates[AddedEmplaceAt]);
					Delegates.RemoveAtSwap(AddedEmplaceAt);
					Delegates.EmplaceAt(FoundIndex, MoveTemp(Element));
				}
				++AddedEmplaceAt;
			}
			else
			{
				Delegates.RemoveAtSwap(AddedEmplaceAt);
			}
		}
		AddedEmplaceAt = std::numeric_limits<uint16>::max();

		DelegateLockCount = 0;
	}
}


TArray<FFieldMulticastDelegate::FDelegateView> FFieldMulticastDelegate::GetView() const
{
	TArray<FFieldMulticastDelegate::FDelegateView> Result;
	Result.Reserve(Delegates.Num());
	for (const FInvocationElement& Element : Delegates)
	{
		if (Element.Delegate.IsBound())
		{
			Result.Emplace(Element.Key.Object.Get(), Element.Key.Id, Element.Delegate.GetUObject(), Element.Key.DynamicName);
		}
	}
	return Result;
}

} //namespace