// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "FieldNotificationDelegate.h"


#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "FieldNotificationMulticastDelegateTest"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFieldNotificationMulticastDelegateTest, "System.UMG.FieldNotificationMulticastDelegateTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);


bool FFieldNotificationMulticastDelegateTest::RunTest(const FString& Parameters)
{
	struct FLocal
	{
		FLocal(FFieldNotificationMulticastDelegateTest* Value)
			: Self(Value)
		{}
		FFieldNotificationMulticastDelegateTest* Self = nullptr;
		UE::FieldNotification::FFieldMulticastDelegate Delegate;
		int32 FooACount = 0;
		int32 FooBCount = 0;
		int32 FooCCount = 0;
		UE::FieldNotification::FFieldId ExpectFieldA;
		UE::FieldNotification::FFieldId ExpectFieldB;
		UE::FieldNotification::FFieldId ExpectFieldC;

		void Reset()
		{
			FooACount = 0;
			FooBCount = 0;
			FooCCount = 0;
			ExpectFieldA = UE::FieldNotification::FFieldId();
			ExpectFieldB = UE::FieldNotification::FFieldId();
			ExpectFieldC = UE::FieldNotification::FFieldId();
		}
		void FooA(UObject* InObject, UE::FieldNotification::FFieldId Id)
		{
			++FooACount;
			if (ExpectFieldA.IsValid() && Id != ExpectFieldA)
			{
				Self->AddError(TEXT("The expected field A is not valid."));
			}
		}
		void FooB(UObject* InObject, UE::FieldNotification::FFieldId Id)
		{
			++FooBCount;
			if (ExpectFieldB.IsValid() && Id != ExpectFieldB)
			{
				Self->AddError(TEXT("The expected field B is not valid."));
			}
		}
		void FooC(UObject* InObject, UE::FieldNotification::FFieldId Id)
		{
			++FooCCount;
			if (ExpectFieldC.IsValid() && Id != ExpectFieldC)
			{
				Self->AddError(TEXT("The expected field C is not valid."));
			}
		}
	};

	UE::FieldNotification::FFieldId FieldA{ "FieldA", 0 };
	UE::FieldNotification::FFieldId FieldB{ "FieldB", 1 };
	UE::FieldNotification::FFieldId FieldC{ "FieldC", 2 };
	UE::FieldNotification::FFieldId FieldD{ "FieldD", 3 };

	FLocal Local{ this };
	auto RunTest1 = [&Local, &FieldA, &FieldB, &FieldC, this]()
	{
		{
			Local.Reset();
			FDelegateHandle Handle1 = Local.Delegate.Add(UObject::StaticClass(), FieldA, UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateRaw(&Local, &FLocal::FooA));
			Local.ExpectFieldA = FieldA;
			Local.Delegate.Broadcast(UObject::StaticClass(), FieldA);
			AddErrorIfFalse(Local.FooACount == 1, TEXT("Bad amount of FooA"));
			AddErrorIfFalse(Local.FooBCount == 0, TEXT("Bad amount of FooB"));

			Local.Reset();
			FDelegateHandle Handle2 = Local.Delegate.Add(UObject::StaticClass(), FieldA, UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateRaw(&Local, &FLocal::FooA));
			Local.ExpectFieldA = FieldA;
			Local.Delegate.Broadcast(UObject::StaticClass(), FieldA);
			AddErrorIfFalse(Local.FooACount == 2, TEXT("Bad amount of FooA"));
			AddErrorIfFalse(Local.FooBCount == 0, TEXT("Bad amount of FooB"));

			UE::FieldNotification::FFieldMulticastDelegate::FRemoveFromResult Result1 = Local.Delegate.RemoveFrom(UObject::StaticClass(), FieldA, Handle1);
			AddErrorIfFalse(Result1.bRemoved, TEXT("Bad Remove of Handle1"));
			AddErrorIfFalse(Result1.bHasOtherBoundDelegates, TEXT("Remove of Handle1 should have other bound delegates."));
			UE::FieldNotification::FFieldMulticastDelegate::FRemoveFromResult Result2 = Local.Delegate.RemoveFrom(UObject::StaticClass(), FieldA, Handle1);
			AddErrorIfFalse(!Result2.bRemoved, TEXT("Bad Remove of Handle1"));
			AddErrorIfFalse(Result2.bHasOtherBoundDelegates, TEXT("Remove of Handle1 should not have other bound delegates."));
			UE::FieldNotification::FFieldMulticastDelegate::FRemoveFromResult Result3 = Local.Delegate.RemoveFrom(UObject::StaticClass(), FieldA, Handle2);
			AddErrorIfFalse(Result3.bRemoved, TEXT("Bad Remove of Handle2"));
			AddErrorIfFalse(!Result3.bHasOtherBoundDelegates, TEXT("Remove of Handle2 should not have other bound delegates."));
		}

		{
			Local.Reset();
			FDelegateHandle Handle1 = Local.Delegate.Add(UObject::StaticClass(), FieldA, UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateRaw(&Local, &FLocal::FooA));
			FDelegateHandle Handle2 = Local.Delegate.Add(UClass::StaticClass(), FieldA, UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateRaw(&Local, &FLocal::FooA));
			Local.Delegate.Add(UClass::StaticClass(), FieldB, UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateRaw(&Local, &FLocal::FooB));
			Local.Delegate.Add(UClass::StaticClass(), FieldC, UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateRaw(&Local, &FLocal::FooC));
			Local.ExpectFieldA = FieldA;
			Local.ExpectFieldB = FieldB;
			Local.ExpectFieldC = FieldC;
			Local.Delegate.Broadcast(UObject::StaticClass(), FieldA);
			AddErrorIfFalse(Local.FooACount == 1, TEXT("Bad amount of FooA"));
			AddErrorIfFalse(Local.FooBCount == 0, TEXT("Bad amount of FooB"));
			AddErrorIfFalse(Local.FooCCount == 0, TEXT("Bad amount of FooC"));
			Local.Delegate.Broadcast(UClass::StaticClass(), FieldA);
			AddErrorIfFalse(Local.FooACount == 2, TEXT("Bad amount of FooA"));
			AddErrorIfFalse(Local.FooBCount == 0, TEXT("Bad amount of FooB"));
			AddErrorIfFalse(Local.FooCCount == 0, TEXT("Bad amount of FooC"));
			Local.Delegate.Broadcast(UClass::StaticClass(), FieldB);
			AddErrorIfFalse(Local.FooACount == 2, TEXT("Bad amount of FooA"));
			AddErrorIfFalse(Local.FooBCount == 1, TEXT("Bad amount of FooB"));
			AddErrorIfFalse(Local.FooCCount == 0, TEXT("Bad amount of FooC"));
			Local.Delegate.Broadcast(UClass::StaticClass(), FieldC);
			AddErrorIfFalse(Local.FooACount == 2, TEXT("Bad amount of FooA"));
			AddErrorIfFalse(Local.FooBCount == 1, TEXT("Bad amount of FooB"));
			AddErrorIfFalse(Local.FooCCount == 1, TEXT("Bad amount of FooC"));

			UE::FieldNotification::FFieldMulticastDelegate::FRemoveResult Result1 = Local.Delegate.Remove(Handle2);
			AddErrorIfFalse(Result1.Object == UClass::StaticClass(), TEXT("Failed Remove Handle2, bad Object"));
			AddErrorIfFalse(Result1.FieldId == FieldA, TEXT("Failed Remove Handle2, bad FieldId"));
			AddErrorIfFalse(Result1.bRemoved, TEXT("Failed Remove Handle2, bad bRemoved"));
			AddErrorIfFalse(!Result1.bHasOtherBoundDelegates, TEXT("Failed Remove Handle2, bad bHasOtherBoundDelegates"));

			UE::FieldNotification::FFieldMulticastDelegate::FRemoveAllResult Result2 = Local.Delegate.RemoveAll(UClass::StaticClass(), &Local);
			AddErrorIfFalse(Result2.RemoveCount == 2, TEXT("Failed RemoveAll, bad count"));
			AddErrorIfFalse(Result2.HasFields.CountSetBits() == 0, TEXT("Failed RemoveAll, bad HasFields"));

			UE::FieldNotification::FFieldMulticastDelegate::FRemoveFromResult Result3 = Local.Delegate.RemoveFrom(UObject::StaticClass(), FieldA, Handle1);
			AddErrorIfFalse(Result3.bRemoved, TEXT("Bad Remove of FieldA"));
			AddErrorIfFalse(!Result3.bHasOtherBoundDelegates, TEXT("Remove of Handle1 should have other bound delegates."));
		}
	};
	RunTest1();

	Local.Delegate.Add(
			UObject::StaticClass(),
			FieldD,
			UE::FieldNotification::FFieldMulticastDelegate::FDelegate::CreateLambda([&RunTest1](UObject* InObject, UE::FieldNotification::FFieldId Id)
			{
				RunTest1();
			}));
	Local.Delegate.Broadcast(UObject::StaticClass(), FieldD);

	return true;
}

#undef LOCTEXT_NAMESPACE 
#endif //WITH_AUTOMATION_WORKER