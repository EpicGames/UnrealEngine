// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FieldNotificationId.h"
#include "IFieldNotificationClassDescriptor.h"


/*
 *	struct FFieldNotificationClassDescriptor : Super::FFieldNotificationClassDescriptor
 *	{
 *	private:
 *		using SuperDescriptor = Super::FFieldNotificationClassDescriptor;
 *		static const ::UE::FieldNotification::FFieldId* AllFields[];
 *		friend ThisClass;
 *	public:
 *		static const ::UE::FieldNotification::FFieldId FieldA;
 *		static const ::UE::FieldNotification::FFieldId FieldB;
 *		enum
 *		{
 *			IndexOf_FieldA = Super::FFieldNotificationClassDescriptor::Max_IndexOf_ + 0,
 *			IndexOf_FieldB,
 *			Max_IndexOf_,
 *		};
 * 	virtual void ForEachField(const UClass* Class, TFunctionRef<bool(::UE::FieldNotification::FFieldId FielId)> Callback) const override;
 *	};
 * virtual const ::UE::FieldNotification::IClassDescriptor& GetFieldNotificationDescriptor() const override;
 */

#define UE_FIELD_NOTIFICATION_OVERRIDE_GET_FIELD_NOTIFICATION_DESCRIPTOR() \
	virtual const ::UE::FieldNotification::IClassDescriptor& GetFieldNotificationDescriptor() const \
	{ \
 		return FFieldNotificationClassDescriptor::GetDescriptor(); \
 	}

#define UE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_BEGIN(API_STRING) \
	struct FFieldNotificationClassDescriptor : public Super::FFieldNotificationClassDescriptor \
	{ \
	private: \
		using SuperDescriptor = Super::FFieldNotificationClassDescriptor; \
		static API_STRING const ::UE::FieldNotification::FFieldId* AllFields[]; \
		static API_STRING const ::UE::FieldNotification::IClassDescriptor& GetDescriptor(); \
		friend ThisClass; \
	public:


#define UE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_END() \
		virtual void ForEachField(const UClass* Class, TFunctionRef<bool(::UE::FieldNotification::FFieldId FielId)> Callback) const override \
		{ \
			for (int32 Index = 0; Index < Max_IndexOf_-SuperDescriptor::Max_IndexOf_; ++Index) \
			{ \
				if (!Callback(*AllFields[Index])) \
				{ \
					return; \
				} \
			} \
			SuperDescriptor::ForEachField(Class, Callback); \
		} \
	}; \
	UE_FIELD_NOTIFICATION_OVERRIDE_GET_FIELD_NOTIFICATION_DESCRIPTOR()


#define UE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_BASE_BEGIN(API_STRING) \
	struct FFieldNotificationClassDescriptor : public ::UE::FieldNotification::IClassDescriptor \
	{ \
	private: \
		using SuperDescriptor = ::UE::FieldNotification::IClassDescriptor; \
		static API_STRING const ::UE::FieldNotification::FFieldId* AllFields[]; \
		static API_STRING const ::UE::FieldNotification::IClassDescriptor& GetDescriptor(); \
		friend ThisClass; \
	public: \
		API_STRING virtual void ForEachField(const UClass* Class, TFunctionRef<bool(::UE::FieldNotification::FFieldId FielId)> Callback) const override; \


#define UE_FIELD_NOTIFICATION_DECLARE_CLASS_DESCRIPTOR_BASE_END() \
	}; \
	UE_FIELD_NOTIFICATION_OVERRIDE_GET_FIELD_NOTIFICATION_DESCRIPTOR()


#define UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD_BEGIN(Name) \
		enum EField \
		{ \
			IndexOf_##Name = SuperDescriptor::Max_IndexOf_ + 0,


#define UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD_END() \
			Max_IndexOf_, \
		};


#define UE_FIELD_NOTIFICATION_DECLARE_FIELD(Name, API_STRING) \
	static API_STRING const ::UE::FieldNotification::FFieldId Name;


#define UE_FIELD_NOTIFICATION_DECLARE_ENUM_FIELD(Name) \
			IndexOf_##Name,


#define UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	const ::UE::FieldNotification::IClassDescriptor& ClassName::FFieldNotificationClassDescriptor::GetDescriptor() \
	{ \
 		static FFieldNotificationClassDescriptor Instance; \
 		return Instance; \
 	} \
	const ::UE::FieldNotification::FFieldId* ClassName::FFieldNotificationClassDescriptor::AllFields[] = {


#define UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName) }

//#if DO_CHECK
//#define UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName) \
//	}; \
//	const ::UE::FieldNotification::IClassDescriptor& ClassName::GetFieldNotificationDescriptor() const \
//	{ \
//		static_assert(UE_ARRAY_COUNT(ClassName::FFieldNotificationClassDescriptor::AllFields) == ClassName::FFieldNotificationClassDescriptor::Max_IndexOf_-ClassName::FFieldNotificationClassDescriptor::SuperDescriptor::Max_IndexOf_, "The descriptor for class " #ClassName " doesn't not implement the same number of field as declared in in the constructor."); \
//		struct FLocal \
//		{ \
//			ClassName::FFieldNotificationClassDescriptor Instance; \
//			FLocal() \
//			{ \
//				int32 NumberOfField = Instance.GetNumberOfField(); \
//				for (int32 Index = 0; Index < NumberOfField; ++Index) \
//				{ \
//					::UE::FieldNotification::FFieldId Id = Instance.GetField(Index); \
//					ensureAlwaysMsgf(Id.IsValid(), TEXT("The FFieldId '%d' for the class '%s' is invalid"), Index, *ClassName::StaticClass()->GetName()); \
//					ensureAlwaysMsgf(Id.GetIndex() == Index, TEXT("The FFieldId at index '%d' doesn't match the id index '%d'."), Index, Id.GetIndex()); \
//				} \
//			} \
//		}; \
//		static FLocal Local; \
//		return Local.Instance; \
//	}
//#else
//#define UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName) \
//	}; \
//	const ::UE::FieldNotification::IClassDescriptor& ClassName::GetFieldNotificationDescriptor() const \
//	{ \
//		static FFieldNotificationClassDescriptor Instance; \
//		return Instance; \
//	}
//#endif


#define UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, MemberName) const ::UE::FieldNotification::FFieldId ClassName::FFieldNotificationClassDescriptor::MemberName( FName(TEXT(#MemberName)), ClassName::FFieldNotificationClassDescriptor::IndexOf_##MemberName );
#define UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, MemberName) &ClassName::FFieldNotificationClassDescriptor::MemberName,


#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_OneField(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)

#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_TwoFields(ClassName, Field1, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)

#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_ThreeFields(ClassName, Field1, Field2, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)

#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_FourFields(ClassName, Field1, Field2, Field3, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)

#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_FiveFields(ClassName, Field1, Field2, Field3, Field4, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)

#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_SixFields(ClassName, Field1, Field2, Field3, Field4, Field5, Field6) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field6) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field6) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)

#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_SevenFields(ClassName, Field1, Field2, Field3, Field4, Field5, Field6, Field7) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field6) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field7) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field6) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field7) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)

#define UE_FIELD_NOTIFICATION_IMPLEMENT_CLASS_DESCRIPTOR_EightFields(ClassName, Field1, Field2, Field3, Field4, Field5, Field6, Field7, Field8) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field6) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field7) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_FIELD(ClassName, Field8) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_BEGIN(ClassName) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field1) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field2) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field3) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field4) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field5) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field6) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field7) \
	UE_FIELD_NOTIFICATION_IMPLEMENT_ENUM_FIELD(ClassName, Field8) \
	UE_FIELD_NOTIFICATION_IMPLEMENTATION_END(ClassName)
