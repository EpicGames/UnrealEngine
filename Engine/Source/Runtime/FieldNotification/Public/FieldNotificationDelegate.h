// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "FieldNotificationId.h"
#include "INotifyFieldValueChanged.h"
#include "Templates/IdentityFunctor.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

#include <limits>

class UObject;

namespace UE::FieldNotification
{
	
class FFieldMulticastDelegate
{
public:
	using FDelegate = INotifyFieldValueChanged::FFieldValueChangedDelegate;
	using FDynamicDelegate = FFieldValueChangedDynamicDelegate;

private:
	struct FInvocationKey
	{
		TWeakObjectPtr<const UObject> Object;
		FFieldId Id;
		FName DynamicName;
		bool operator<(const FInvocationKey& Element) const
		{
			return Id.GetName().FastLess(Element.Id.GetName());
		}
	};

	struct FInvocationElement
	{
		FInvocationKey Key;
		FDelegate Delegate;
		bool operator<(const FInvocationElement& InElement) const
		{
			return Key < InElement.Key;
		}
	};

	friend bool operator<(const FInvocationElement& A, const FInvocationKey& B)
	{
		return A.Key < B;
	}
	friend bool operator<(const FInvocationKey& A, const FInvocationElement& B)
	{
		return A < B.Key;
	}
	friend bool operator<(const FInvocationElement& A, const FFieldId& B)
	{
		return A.Key.Id.GetName().FastLess(B.GetName());
	}
	friend bool operator<(const FFieldId& A, const FInvocationElement& B)
	{
		return A.GetName().FastLess(B.Key.Id.GetName());
	}

	using InvocationListType = TArray<FInvocationElement>;

public:
	FIELDNOTIFICATION_API FDelegateHandle Add(const UObject* InObject, FFieldId InFieldId, FDelegate InNewDelegate);
	FIELDNOTIFICATION_API FDelegateHandle Add(const UObject* InObject, FFieldId InFieldId, const FDynamicDelegate& InDynamicDelegate);

	struct FRemoveResult
	{
		bool bRemoved = false;
		bool bHasOtherBoundDelegates = false;
		const UObject* Object = nullptr;
		FFieldId FieldId;
	};
	FIELDNOTIFICATION_API FRemoveResult Remove(FDelegateHandle InDelegate);
	FIELDNOTIFICATION_API FRemoveResult Remove(const FDynamicDelegate& InDynamicDelegate);

	struct FRemoveFromResult
	{
		bool bRemoved = false;
		bool bHasOtherBoundDelegates = false;
	};
	FIELDNOTIFICATION_API FRemoveFromResult RemoveFrom(const UObject* InObject, FFieldId InFieldId, FDelegateHandle InDelegate);
	FIELDNOTIFICATION_API FRemoveFromResult RemoveFrom(const UObject* InObject, FFieldId InFieldId, const FDynamicDelegate& InDynamicDelegate);

	struct FRemoveAllResult
	{
		int32 RemoveCount = 0;
		TBitArray<> HasFields;
	};
	FIELDNOTIFICATION_API FRemoveAllResult RemoveAll(const UObject* InObject, FDelegateUserObjectConst InUserObject);
	FIELDNOTIFICATION_API FRemoveAllResult RemoveAll(const UObject* InObject, FFieldId InFieldId, FDelegateUserObjectConst InUserObject);

	FIELDNOTIFICATION_API void Broadcast(UObject* InObject, FFieldId InFieldId);

	FIELDNOTIFICATION_API void Reset();

	struct FDelegateView
	{
		FDelegateView(const UObject* InKeyObj, FFieldId InKeyId, const UObject* InObj, FName InFunction)
			: KeyObject(InKeyObj), KeyField(InKeyId), BindingObject(InObj), BindingFunctionName(InFunction)
		{}
		const UObject* KeyObject;
		FFieldId KeyField;
		const UObject* BindingObject;
		FName BindingFunctionName;
	};
	FIELDNOTIFICATION_API TArray<FDelegateView> GetView() const;

private:
	int32 LowerBound(FFieldId InFieldId) const
	{
		int32 Num = AddedEmplaceAt > GetNum(Delegates) ? GetNum(Delegates) : AddedEmplaceAt;
		return AlgoImpl::LowerBoundInternal(GetData(Delegates), Num, InFieldId, FIdentityFunctor(), TLess<>());
	}

	int32 LowerBound(const FInvocationKey& InKey) const
	{
		int32 Num = AddedEmplaceAt > GetNum(Delegates) ? GetNum(Delegates) : AddedEmplaceAt;
		return AlgoImpl::LowerBoundInternal(GetData(Delegates), Num, InKey, FIdentityFunctor(), TLess<>());
	}

	int32 UpperBound(FFieldId InFieldId) const
	{
		int32 Num = AddedEmplaceAt > GetNum(Delegates) ? GetNum(Delegates) : AddedEmplaceAt;
		return AlgoImpl::UpperBoundInternal(GetData(Delegates), Num, InFieldId, FIdentityFunctor(), TLess<>());
	}

	int32 UpperBound(const FInvocationKey& InKey) const
	{
		int32 Num = AddedEmplaceAt > GetNum(Delegates) ? GetNum(Delegates) : AddedEmplaceAt;
		return AlgoImpl::UpperBoundInternal(GetData(Delegates), Num, InKey, FIdentityFunctor(), TLess<>());
	}

	void ExecuteLockOperations();
	void RemoveElement(FInvocationElement& Element, int32 Index, FRemoveResult& Result);
	void CompleteRemove(FRemoveResult& Result) const;
	template<typename TRemoveOrUnbind>
	void RemoveFrom(TRemoveOrUnbind& RemoveOrUnbind, FFieldId FieldId, bool& bFieldPresent);

private:
	InvocationListType Delegates;
	int16 DelegateLockCount = 0;
	int16 CompactionCount = 0;
	uint16 AddedEmplaceAt = std::numeric_limits<uint16>::max();
};

} //namespace
