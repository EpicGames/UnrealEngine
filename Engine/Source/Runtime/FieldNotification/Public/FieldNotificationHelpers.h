// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"

namespace UE::FieldNotification::Helpers
{
	[[nodiscard]] inline bool IsValidAsField(const UFunction* InFunction)
	{
		return InFunction != nullptr
			&& !InFunction->HasAnyFunctionFlags(FUNC_Net | FUNC_Event)
			&& InFunction->HasAllFunctionFlags(FUNC_BlueprintCallable | FUNC_Const)
			&& InFunction->NumParms == 1
			&& InFunction->GetReturnProperty() != nullptr;
	}

} //namespace
