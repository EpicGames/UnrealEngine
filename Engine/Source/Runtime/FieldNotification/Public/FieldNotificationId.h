// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FieldNotificationVariant.h"
#include "UObject/Object.h"

#include "FieldNotificationId.generated.h"


namespace UE::FieldNotification
{
	
struct FFieldId
{
public:
	FFieldId() = default;

	explicit FFieldId(FName InFieldName, int32 InBitNumber)
		: FieldName(InFieldName)
		, BitNumber(InBitNumber)
	{
	}


public:
	bool IsValid() const
	{
		return !FieldName.IsNone();
	}

	int32 GetIndex() const
	{
		return BitNumber;
	}
	
	FName GetName() const
	{
		return FieldName;
	}

	FFieldVariant ToVariant(UObject* InContainer) const
	{
		if (InContainer && IsValid())
		{
			if (UFunction* Function = InContainer->GetClass()->FindFunctionByName(FieldName))
			{
				return FFieldVariant(Function);
			}
			else if (FProperty* Property = InContainer->GetClass()->FindPropertyByName(FieldName))
			{
				return FFieldVariant(Property);
			}
		}
		return FFieldVariant();
	}

	bool operator==(const FFieldId& Other) const
	{
		return FieldName == Other.FieldName;
	}

	bool operator!=(const FFieldId& Other) const
	{
		return FieldName != Other.FieldName;
	}

	friend int32 GetTypeHash(const FFieldId& Value)
	{
		return GetTypeHash(Value.FieldName);
	}

private:
	/** Name of the field. It can be a FProperty or UFunction. */
	FName FieldName;
	/** The bit this field is linked to. */
	int32 BitNumber;
};

} //namespace


USTRUCT(BlueprintType)
struct FFieldNotificationId
{
	GENERATED_BODY()

public:
	FFieldNotificationId() = default;

	explicit FFieldNotificationId(FName InFieldName)
		: FieldName(InFieldName)
	{

	}

public:
	bool IsValid() const
	{
		return !FieldName.IsNone();
	}

	FName GetFieldName() const
	{
		return FieldName;
	}

	bool operator== (const FFieldNotificationId& Other) const
	{
		return FieldName == Other.FieldName;
	}

	bool operator!= (const FFieldNotificationId& Other) const
	{
		return FieldName != Other.FieldName;
	}

	friend int32 GetTypeHash(const FFieldNotificationId& Value)
	{
		return GetTypeHash(Value.FieldName);
	}

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "FieldNotify")
	FName FieldName;
};
