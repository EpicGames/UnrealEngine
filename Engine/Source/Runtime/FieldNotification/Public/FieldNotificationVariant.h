// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"

namespace UE::FieldNotification
{

struct FFieldVariant : private ::FFieldVariant
{
public:
	FFieldVariant() = default;

	explicit FFieldVariant(const FProperty* InProperty)
		: ::FFieldVariant(InProperty)
	{
	}

	explicit FFieldVariant(const UFunction* InFunction)
		: ::FFieldVariant(InFunction)
	{
	}

public:
	using ::FFieldVariant::IsValid;
	using ::FFieldVariant::GetOwnerClass;
	using ::FFieldVariant::GetFName;
	using ::FFieldVariant::operator bool;
	using ::FFieldVariant::operator ==;
	using ::FFieldVariant::operator !=;
#if WITH_METADATA
	using ::FFieldVariant::HasMetaData;
#endif // WITH_METADATA

	inline bool IsProperty() const
	{
		return IsValid() && !IsUObject();
	}

	inline bool IsFunction() const
	{
		return IsValid() && IsUObject();
	}

	inline FProperty* GetProperty()
	{
		return IsProperty() ? static_cast<FProperty*>(ToFieldUnsafe()) : nullptr;
	}
	
	inline const FProperty* GetProperty() const
	{
		return IsProperty() ? static_cast<const FProperty*>(ToFieldUnsafe()) : nullptr;
	}

	inline UFunction* GetFunction()
	{
		return IsFunction() ? static_cast<UFunction*>(ToUObjectUnsafe()) : nullptr;
	}

	inline const UFunction* GetFunction() const
	{
		return IsFunction() ? static_cast<const UFunction*>(ToUObjectUnsafe()) : nullptr;
	}
};

} //namespace
