// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FieldNotificationId.h"

namespace UE::FieldNotification
{

struct IClassDescriptor
{
	enum
	{
		Max_IndexOf_ = 0,
	};

	/** Find the FieldId by name. Returns an invalid FFIeldId if not found. */
	FFieldId GetField(const UClass* Class, FName InFieldName) const
	{
		FFieldId FoundId;
		ForEachField(Class, [&FoundId, InFieldName](const FFieldId Other)
		{
			if (Other.GetName() == InFieldName)
			{
				FoundId = Other;
				return false;
			}
			return true;
		});
		return FoundId;
	}

	/** Execute the callback for every FieldId in the ClassDescriptor. */
	virtual void ForEachField(const UClass* Class, TFunctionRef<bool(FFieldId FielId)> Callback) const
	{

	}

	/** */
	virtual ~IClassDescriptor() = default;
};

} // namespace