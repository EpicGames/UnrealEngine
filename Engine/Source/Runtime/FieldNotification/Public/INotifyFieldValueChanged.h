// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FieldNotificationId.h"
#include "IFieldNotificationClassDescriptor.h"
#include "UObject/Interface.h"

#include "INotifyFieldValueChanged.generated.h"

DECLARE_DYNAMIC_DELEGATE_TwoParams(FFieldValueChangedDynamicDelegate, UObject*, Object, FFieldNotificationId, Field);


UINTERFACE(MinimalAPI, NotBlueprintable)
class UNotifyFieldValueChanged : public UInterface
{
	GENERATED_BODY()
};


class INotifyFieldValueChanged : public IInterface
{
	GENERATED_BODY()

public:
	// using "not checked" user policy (means race detection is disabled) because this delegate is stored in a container and causes its reallocation
	// from inside delegate's execution. This is incompatible with race detection that needs to access the delegate instance after its execution
	using FFieldValueChangedDelegate = TDelegate<void(UObject*, UE::FieldNotification::FFieldId), FNotThreadSafeNotCheckedDelegateUserPolicy>;

public:
	/** Add a delegate that will be notified when the FieldId is value changed. */
	virtual FDelegateHandle AddFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FFieldValueChangedDelegate InNewDelegate) = 0;

	/** Remove a delegate that was added. */
	virtual bool RemoveFieldValueChangedDelegate(UE::FieldNotification::FFieldId InFieldId, FDelegateHandle InHandle) = 0;

	/** Remove all the delegate that are bound to the specified UserObject. */
	virtual int32 RemoveAllFieldValueChangedDelegates(FDelegateUserObjectConst InUserObject) = 0;

	/** Remove all the delegate that are bound to the specified Field and UserObject. */
	virtual int32 RemoveAllFieldValueChangedDelegates(UE::FieldNotification::FFieldId InFieldId, FDelegateUserObjectConst InUserObject) = 0;

	/** @returns the list of all the field that can notify when their value changes. */
	virtual const UE::FieldNotification::IClassDescriptor& GetFieldNotificationDescriptor() const = 0;

	/** Broadcast to the registered delegate that the FieldId value changed. */
	virtual void BroadcastFieldValueChanged(UE::FieldNotification::FFieldId InFieldId) = 0;
};
